// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AudioFormatBink : ModuleRules
{
	public AudioFormatBink(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine"
			}
		);

		string SdkDir = Path.Combine(ModuleDirectory, "..", "..", "Runtime", "BinkAudioDecoder", "SDK", "BinkAudio");
		
		string LibName = null;
		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			LibName = (Target.Architecture == UnrealArch.Arm64) ? "binka_ue_encode_winarm64_static.lib" : "binka_ue_encode_win64_static.lib";
		}
		if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			LibName = "libbinka_ue_encode_lnx64_static.a";
		}
		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			LibName = "libbinka_ue_encode_osx_static.a";
		}

		PublicSystemIncludePaths.Add(Path.Combine(SdkDir, "Include"));
		if (LibName != null)
		{
			PublicAdditionalLibraries.Add(Path.Combine(SdkDir, "Lib", LibName));
		}
	}
}
