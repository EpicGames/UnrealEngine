// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"
#include "HAL/Platform.h"

#include "binka_ue_file_header.h"
#include "binka_ue_encode.h"

static const FName NAME_BINKA(TEXT("BINKA"));

DEFINE_LOG_CATEGORY_STATIC(LogAudioFormatBink, Display, All);

namespace AudioFormatBinkPrivate
{
	// Taken from BinkAudioInfo.cpp 
	static uint32 GetMaxFrameSizeSamples(const uint32 SampleRate)
	{
		if (SampleRate >= 44100)
		{
			return 1920;
		}
		else if (SampleRate >= 22050)
		{
			return 960;
		}
		else
		{
			return 480;
		}
	}

	static uint8 GetCompressionLevelFromQualityIndex(const int32 InQualityIndex) 
	{
		// Bink goes from 0 (best) to 9 (worst), but is basically unusable below 4 
		static constexpr float BinkLowest = 4;
		static constexpr float BinkHighest = 0;

		// Map Quality 1 (lowest) to 100 (highest).
		static constexpr float QualityLowest = 1;
		static constexpr float QualityHighest = 100;

		// Map Quality into Bink Range. Note: +1 gives the Bink range 5 steps inclusive.
		const float BinkValue = FMath::GetMappedRangeValueClamped(FVector2D(QualityLowest, QualityHighest), FVector2D(BinkLowest + 1.f, BinkHighest), InQualityIndex);

		// Floor each value and clamp into range (as top lerp will be +1 over)
		return FMath::Clamp(FMath::FloorToInt(BinkValue), BinkHighest, BinkLowest);
	}	

	static uint16 GetMaxSeekTableEntries(const FSoundQualityInfo& InQualityInfo)
	{
		const uint64 DurationFrames = InQualityInfo.SampleDataSize / (InQualityInfo.NumChannels * sizeof(int16));
		const uint64 MaxEntries = DurationFrames / GetMaxFrameSizeSamples(InQualityInfo.SampleRate);
				
		static constexpr uint16 BinkDefault = 4096;
		static constexpr uint16 BinkMax		= TNumericLimits<uint16>::Max();

		const uint64 Clamped = FMath::Clamp<uint64>(MaxEntries, BinkDefault, BinkMax);
			
		return IntCastChecked<uint16>(Clamped);
	}
}

/**
 * IAudioFormat, audio compression abstraction
**/
class FAudioFormatBink : public IAudioFormat
{
	enum
	{
		/** Version for Bink Audio format, this becomes part of the DDC key. */
		UE_AUDIO_BINK_VER = 11,
	};

public:
	virtual bool AllowParallelBuild() const override
	{
		return true;
	}

	virtual uint16 GetVersion(FName Format) const override
	{
		check(Format == NAME_BINKA);
		return UE_AUDIO_BINK_VER;
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(NAME_BINKA);
	}

	static void* BinkAlloc(const size_t Bytes)
	{
		return FMemory::Malloc(Bytes, 16);
	}
	static void BinkFree(void* Ptr)
	{
		FMemory::Free(Ptr);
	}

	virtual bool Cook(FName InFormat, const TArray<uint8>& InSrcBuffer, FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutCompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatBink::Cook);
		check(InFormat == NAME_BINKA);

		const uint8 CompressionLevel = AudioFormatBinkPrivate::GetCompressionLevelFromQualityIndex(InQualityInfo.Quality);
		
		// If we're going to embed the seek-table in the stream, use -1 to give the largest table we can produce.
		const uint16 MaxSeektableSize = AudioFormatBinkPrivate::GetMaxSeekTableEntries(InQualityInfo);
		
		void* CompressedData = 0;
		uint32_t CompressedDataLen = 0;
		uint8_t BinkCompressError = UECompressBinkAudio((void*)InSrcBuffer.GetData(), InSrcBuffer.Num(), InQualityInfo.SampleRate, InQualityInfo.NumChannels, CompressionLevel, 1, MaxSeektableSize, BinkAlloc, BinkFree, &CompressedData, &CompressedDataLen);

		const TCHAR* CompressErrorStr = nullptr;
		switch (BinkCompressError)
		{
		case BINKA_COMPRESS_SUCCESS: break;
		case BINKA_COMPRESS_ERROR_CHANS: CompressErrorStr = TEXT("Invalid channel count, max ") BINKA_MAX_CHANS_STR; break;
		case BINKA_COMPRESS_ERROR_SAMPLES: CompressErrorStr = TEXT("No sample data provided"); break;
		case BINKA_COMPRESS_ERROR_RATE: CompressErrorStr = TEXT("Invalid sample rate provided, min ") BINKA_MIN_RATE_STR TEXT(" max ") BINKA_MAX_RATE_STR; break;
		case BINKA_COMPRESS_ERROR_QUALITY: CompressErrorStr = TEXT("Invalid quality provided, valid is 0-9"); break;
		case BINKA_COMPRESS_ERROR_ALLOCATORS: CompressErrorStr = TEXT("No allocators provided!"); break;
		case BINKA_COMPRESS_ERROR_OUTPUT: CompressErrorStr = TEXT("No output pointers provided!"); break;
		case BINKA_COMPRESS_ERROR_SEEKTABLE: CompressErrorStr = TEXT("Invalid seektable size limit specified!"); break;
		case BINKA_COMPRESS_ERROR_SIZE: CompressErrorStr = TEXT("Input file too big - can't fit offsets in seek table!"); break;
		}

		if (CompressErrorStr != nullptr)
		{
			UE_LOGF(LogAudioFormatBink, Warning, "Failed to encode bink audio: %ls", CompressErrorStr);
			return false;
		}

		// Create a buffer to store compressed data
		OutCompressedDataStore.Empty();
		OutCompressedDataStore.Append((uint8*)CompressedData, CompressedDataLen);
		BinkFree(CompressedData);
		return OutCompressedDataStore.Num() > 0;
	}

	virtual bool CookSurround(FName InFormat, const TArray<TArray<uint8> >& InSrcBuffers, FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutCompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatBink::CookSurround);
		check(InFormat == NAME_BINKA);

		//
		// CookSurround passes us a bunch of mono buffers, but bink audio wants a standard 
		// interleaved buffer
		//
		TArray<uint8> InterleavedSrcBuffers;
		InterleavedSrcBuffers.AddUninitialized(InSrcBuffers[0].Num() * InSrcBuffers.Num());
		int16* Dest = (int16*)InterleavedSrcBuffers.GetData();

		uint32 ChannelCount = InSrcBuffers.Num();
		uint32 FrameCount = InSrcBuffers[0].Num() / sizeof(int16);
		for (uint32 FrameIndex = 0; FrameIndex < FrameCount; FrameIndex++)
		{
			for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ChannelIndex++)
			{
				int16* Src = (int16*)InSrcBuffers[ChannelIndex].GetData();
				Dest[FrameIndex * ChannelCount + ChannelIndex] = Src[FrameIndex];
			}
		}

		return Cook(NAME_BINKA, InterleavedSrcBuffers, InQualityInfo, OutCompressedDataStore);
	}

	// AFAICT this function is never called.
	virtual int32 Recompress(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer) const override
	{
		return 0;
	}

	virtual int32 GetMinimumSizeForInitialChunk(FName Format, const TArray<uint8>& SrcBuffer) const override
	{
		// Exclude any seek table entries in our size, as we are now using streaming seek tables.
		if(RequiresStreamingSeekTable())
		{
			return sizeof(BinkAudioFileHeader);
		}
		
		// We must have an initial chunk large enough for the header and the seek table, if present.
		BinkAudioFileHeader const* Header = (BinkAudioFileHeader const*)SrcBuffer.GetData();
		return sizeof(BinkAudioFileHeader) + Header->seek_table_entry_count * sizeof(uint16);
	}

	// Takes in a compressed file and splits it in to stream size chunks. AFAICT this is supposed to accumulate frames
	// until the chunk size is reached, then spit out a block.
	virtual bool SplitDataForStreaming(const TArray<uint8>& InSrcBuffer, TArray<TArray<uint8>>& OutBuffers, const int32 InMaxInitialChunkSize, const int32 InMaxChunkSize) const override
	{
		// This should not be called if we require a streaming seek-table. 
		if (!ensure(RequiresStreamingSeekTable()==false))
		{
			return false;
		}

		uint8 const* Source = InSrcBuffer.GetData();
		uint32 SourceLen = InSrcBuffer.Num();
		uint8 const* SourceEnd = Source + SourceLen;

		uint8 const* ChunkStart = Source;
		uint8 const* Current = Source;
	
		ensure(SourceLen > sizeof(BinkAudioFileHeader));

		BinkAudioFileHeader const* Header = (BinkAudioFileHeader const*)Source;
		Current += sizeof(BinkAudioFileHeader);

		ensure(Header->tag == 'UEBA');

		// The first frame is located past the seek table.
		Current += sizeof(uint16) * Header->seek_table_entry_count;

		// We must be on the first frame.
		uint16 CurrentFrameValue = *(uint16*)Current;
		ensure(BLOCK_HEADER_MAGIC == CurrentFrameValue);

		int32 ChunkLimitBytes = InMaxInitialChunkSize;
		for (;;)
		{
			if (Current >= SourceEnd)
			{
				// Done with the file.
				check(Current == SourceEnd);
				Current = SourceEnd;
				break;
			}

			// Advance to next chunk
			uint32 BlockSize;
			if (BinkAudioBlockSize(Header->max_comp_space_needed, Current, (SourceEnd - Current), &BlockSize) == false)
			{
				// if this happens, then UE didn't actually give us the data we gave it, and we're
				// off the edge of the map.
				check(0);
				return false;
			}

			if ((Current - ChunkStart) + BlockSize >= ChunkLimitBytes)
			{
				// can't add this chunk, emit.
				TArray<uint8> Chunk(ChunkStart, Current - ChunkStart);
				OutBuffers.Add(Chunk);
				ChunkStart = Current;
				ChunkLimitBytes = InMaxChunkSize;

				// retry.
				continue;
			}

			Current += BlockSize;
		}

		// emit any remainder chunks
		if (Current - ChunkStart)
		{
			// emit this chunk
			TArray<uint8> Chunk(ChunkStart, Current - ChunkStart);
			OutBuffers.Add(Chunk);
			ChunkStart = Current;
		}

		return true;
	}

	static void StripSeek(TArray<uint8>& InOutBuffer)
	{
		void* CompressedData = InOutBuffer.GetData();
		uint32_t CompressedDataLen = InOutBuffer.Num();

		check(CompressedDataLen > sizeof(BinkAudioFileHeader));

		BinkAudioFileHeader* Header = (BinkAudioFileHeader*)CompressedData;
		uint32 SeekTableBytes = Header->seek_table_entry_count * sizeof(uint16);
		check(CompressedDataLen > sizeof(BinkAudioFileHeader) + SeekTableBytes);

		// Mark in the header we don't have any seek table 
		Header->seek_table_entry_count = 0;

		// Copy the rest of the encoded data over the seek table.
		uint8* SeekTableStart = (uint8*)(Header + 1);
		FMemory::Memmove(SeekTableStart, SeekTableStart + SeekTableBytes, CompressedDataLen - SeekTableBytes - sizeof(BinkAudioFileHeader));

		CompressedDataLen -= SeekTableBytes;

		InOutBuffer.SetNum(CompressedDataLen);
	}
	
	virtual bool RequiresStreamingSeekTable() const override
	{
		return true; // Toggling this will require a version bump.
	}

	virtual bool ExtractSeekTableForStreaming(TArray<uint8>& InOutBuffer, IAudioFormat::FSeekTable& OutSeektable) const override
	{
		// This should only be called if we require a streaming seek-table. 
		if (!ensure(RequiresStreamingSeekTable()))
		{
			return false;
		}

		BinkAudioFileHeader const* Header = reinterpret_cast<BinkAudioFileHeader const*>(InOutBuffer.GetData());
		if (InOutBuffer.Num() < sizeof(BinkAudioFileHeader) || Header->tag != 'UEBA' || Header->seek_table_entry_count==0)
		{
			return false;
		}
		
		// The outer logic that manages the seek-table is unaware of how big the header can be, so for the sake of simplicity,
		// offset for the size of the header in the seek-table entries, so we don't need to worry about adjusting for it later.
		static constexpr uint32 ActualAudioOffset = sizeof(BinkAudioFileHeader) + 0; // No entries as we're stripping it.

		// Decode and copy out the seek-table. (it's stored as deltas).
		const uint16* EncodedSeekTable = reinterpret_cast<uint16*>(InOutBuffer.GetData() + sizeof(BinkAudioFileHeader));
		uint32 CurrentSeekOffset = ActualAudioOffset;
		uint32 CurrentTimeOffset = 0;
		
		const int32 SamplesPerEntry = Header->blocks_per_seek_table_entry * AudioFormatBinkPrivate::GetMaxFrameSizeSamples(Header->rate);
		OutSeektable.Offsets.SetNum(Header->seek_table_entry_count);
		OutSeektable.Times.SetNum(Header->seek_table_entry_count);

		for (int32 i = 0; i < Header->seek_table_entry_count; ++i)
		{
			OutSeektable.Times[i] = CurrentTimeOffset;
			OutSeektable.Offsets[i] = CurrentSeekOffset;
			CurrentSeekOffset += EncodedSeekTable[i];
			CurrentTimeOffset += SamplesPerEntry;
		}

		// Check that the last block which spans the last offset in the table and end of file
		// is a reasonable size.
		if (!ensure(InOutBuffer.Num() - OutSeektable.Offsets.Last() < 1024*1024))
		{
			return false;
		}

		// Strip the seek-table from the buffer now we've copied it.
		StripSeek(InOutBuffer);

		return true;
	}
};

class FAudioPlatformBinkModule final : public IAudioFormatModule
{
private:
	FAudioFormatBink* BinkEncoder = nullptr;

public:
	virtual ~FAudioPlatformBinkModule() override {}
	
	virtual IAudioFormat* GetAudioFormat() override
	{
		return BinkEncoder;
	}
	virtual void StartupModule() override
	{
		BinkEncoder = new FAudioFormatBink();
	}
	virtual void ShutdownModule() override
	{
		delete BinkEncoder;
		BinkEncoder = nullptr;
	}
};

IMPLEMENT_MODULE( FAudioPlatformBinkModule, AudioFormatBink);
